// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "CableActor.generated.h"

/** An actor that renders a simulated cable */
UCLASS(MinimalAPI, hidecategories=(Input,Replication), showcategories=("Input|MouseInput", "Input|TouchInput"))
class ACableActor : public AActor
{
	GENERATED_UCLASS_BODY()

	/** Cable component that performs simulation and rendering */
	UPROPERTY(Category=Cable, VisibleAnywhere, BlueprintReadOnly)
	TObjectPtr<class UCableComponent> CableComponent;
};
