// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"
#include "CableComponent.generated.h"

#define UE_API CABLECOMPONENT_API

class FPrimitiveSceneProxy;

/** Struct containing information about a point along the cable */
struct FCableParticle
{
	FCableParticle()
	: bFree(true)
	, Position(0,0,0)
	, OldPosition(0,0,0)
	{}

	/** If this point is free (simulating) or fixed to something */
	bool bFree;
	/** Current position of point */
	FVector Position;
	/** Position of point on previous iteration */
	FVector OldPosition;
};

/** Component that allows you to specify custom triangle mesh geometry */
UCLASS(MinimalAPI, hidecategories=(Object, Physics, Activation, "Components|Activation"), editinlinenew, meta=(BlueprintSpawnableComponent), ClassGroup=Rendering)
class UCableComponent : public UMeshComponent
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UActorComponent Interface.
	UE_API virtual void OnRegister() override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	UE_API virtual void SendRenderDynamicData_Concurrent() override;
	UE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	UE_API virtual void ApplyWorldOffset(const FVector& InOffset, bool bWorldShift) override;
	//~ End UActorComponent Interface.

	//~ Begin USceneComponent Interface.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	UE_API virtual void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const override;
	UE_API virtual bool HasAnySockets() const override;
	UE_API virtual bool DoesSocketExist(FName InSocketName) const override;
	UE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	UE_API virtual void OnVisibilityChanged() override;
	//~ End USceneComponent Interface.

	//~ Begin UPrimitiveComponent Interface.
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;
	//~ End UPrimitiveComponent Interface.

	//~ Begin UMeshComponent Interface.
	UE_API virtual int32 GetNumMaterials() const override;
	//~ End UMeshComponent Interface.


	/**
	 *	Should we fix the start to something, or leave it free.
	 *	If false, component transform is just used for initial location of start of cable
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cable")
	bool bAttachStart;

	/** 
	 *	Should we fix the end to something (using AttachEndTo and EndLocation), or leave it free. 
	 *	If false, AttachEndTo and EndLocation are just used for initial location of end of cable
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cable")
	bool bAttachEnd;

	/** Actor or Component that the defines the end position of the cable */
	UPROPERTY(EditAnywhere, Category="Cable")
	FComponentReference AttachEndTo;

	/** Socket name on the AttachEndTo component to attach to */
	UPROPERTY(EditAnywhere, Category = "Cable")
	FName AttachEndToSocketName;

	/** End location of cable, relative to AttachEndTo (or AttachEndToSocketName) if specified, otherwise relative to cable component. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cable", meta=(MakeEditWidget=true))
	FVector EndLocation;

	/** Attaches the end of the cable to a specific Component **/
	UFUNCTION(BlueprintCallable, Category = "Cable")
	UE_API void SetAttachEndToComponent(USceneComponent* Component, FName SocketName = NAME_None);

	/** Attaches the end of the cable to a specific Component within an Actor **/
	UFUNCTION(BlueprintCallable, Category = "Cable")
	UE_API void SetAttachEndTo(AActor* Actor, FName ComponentProperty, FName SocketName = NAME_None);
	
	/** Gets the Actor that the cable is attached to **/
	UFUNCTION(BlueprintCallable, Category = "Cable")
	UE_API AActor* GetAttachedActor() const;

	/** Gets the specific USceneComponent that the cable is attached to **/
	UFUNCTION(BlueprintCallable, Category = "Cable")
	UE_API USceneComponent* GetAttachedComponent() const;

	/** Get array of locations of particles (in world space) making up the cable simulation. */
	UFUNCTION(BlueprintCallable, Category = "Cable")
	UE_API void GetCableParticleLocations(TArray<FVector>& Locations) const;

	/** Rest length of the cable */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cable", meta=(ClampMin = "0.0", UIMin = "0.0", UIMax = "1000.0"))
	float CableLength;

	/** How many segments the cable has */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Cable", meta=(ClampMin = "1", UIMin = "1", UIMax = "20"))
	int32 NumSegments;

	/** Controls the simulation substep time for the cable */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category="Cable", meta=(ClampMin = "0.005", UIMin = "0.005", UIMax = "0.1"))
	float SubstepTime;

	/** The number of solver iterations controls how 'stiff' the cable is */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cable", meta=(ClampMin = "1", ClampMax = "16"))
	int32 SolverIterations;

	/** Add stiffness constraints to cable. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Cable")
	bool bEnableStiffness;

	/** When false, will still wait for SubstepTime to elapse before updating, but will only run the cable simulation once using all of accumulated simulation time */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Cable")
	bool bUseSubstepping = true;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Cable")
	bool bSkipCableUpdateWhenNotVisible = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Cable")
	bool bSkipCableUpdateWhenNotOwnerRecentlyRendered = false;

	/** 
	 *	EXPERIMENTAL. Perform sweeps for each cable particle, each substep, to avoid collisions with the world. 
	 *	Uses the Collision Preset on the component to determine what is collided with.
	 *	This greatly increases the cost of the cable simulation.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Cable")
	bool bEnableCollision;

	/** If collision is enabled, control how much sliding friction is applied when cable is in contact. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Cable", meta = (ClampMin = "0.0", ClampMax = "1.0", EditCondition = "bEnableCollision"))
	float CollisionFriction;

	/** Force vector (world space) applied to all particles in cable. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cable Forces")
	FVector CableForce;

	/** Scaling applied to world gravity affecting this cable. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Cable Forces")
	float CableGravityScale;

	/** How wide the cable geometry is */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cable Rendering", meta=(ClampMin = "0.01", UIMin = "0.01", UIMax = "50.0"))
	float CableWidth;

	/** Number of sides of the cable geometry */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Cable Rendering", meta=(ClampMin = "1", ClampMax = "16"))
	int32 NumSides;

	/** How many times to repeat the material along the length of the cable */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Cable Rendering", meta=(UIMin = "0.1", UIMax = "8"))
	float TileMaterial;

	/**
	* Always reset cable particle positions and velocities on a teleport.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Teleport")
	bool bResetAfterTeleport = false;

	/**
	* Conduct teleportation if the movement of either fixed end point of the cable is greater than this threshold in 1 frame.
	* Zero or negative values will skip the check.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Teleport", meta=(UIMin = "0"))
	float TeleportDistanceThreshold = 500;

	/**
	* Rotation threshold in degrees.
	* Conduct teleportation if the cable's rotation is greater than this threshold in 1 frame.
	* Zero or negative values will skip the check.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Teleport", meta = (UIMin = "0"))
	float TeleportRotationThreshold = 0;

	/**
	* Teleport the cable particles on reattaching any end of the cable.
	**/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Teleport")
	bool bTeleportAfterReattach = false;

private:

	/** Solve the cable spring constraints */
	UE_API void SolveConstraints();
	/** Integrate cable point positions */
	UE_API void VerletIntegrate(float InSubstepTime, const FVector& Gravity);
	/** Perform collision traces for particles */
	UE_API void PerformCableCollision();
	/** Perform a simulation substep */
	UE_API void PerformSubstep(float InSubstepTime, const FVector& Gravity);
	/** Get start and end position for the cable */
	UE_API void GetEndPositions(FVector& OutStartPosition, FVector& OutEndPosition);
	/** Amount of time 'left over' from last tick */
	float TimeRemainder;
	/** Array of cable particles */
	TArray<FCableParticle>	Particles;

	/** Previous cable state for teleport corrections */
	FTransform LastTransform;
	FTransform LastEndPointTransform;
	FVector LastEndPoint;
	FVector LastEndLocation;
	FVector LastStartPoint;
	bool bLastStartAttached;
	bool bLastEndAttached;
	
	/** Perform checks and corrections on particle positions for teleports */
	UE_API void DoTeleportCorrections(const FVector& StartPosition, const FVector& EndPosition);

	friend class FCableSceneProxy;
};

#undef UE_API
