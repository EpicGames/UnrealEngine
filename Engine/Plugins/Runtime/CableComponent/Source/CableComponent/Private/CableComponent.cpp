// Copyright Epic Games, Inc. All Rights Reserved. 

#include "CableComponent.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "MaterialDomain.h"
#include "PrimitiveDrawingUtils.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "Engine/CollisionProfile.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "Engine/Engine.h"
#include "CableComponentStats.h"
#include "DynamicMeshBuilder.h"
#include "SceneView.h"
#include "StaticMeshResources.h"
#include "RayTracingInstance.h"
#include "SceneInterface.h"

DEFINE_RENDER_COMMAND_PIPE(Cable, ERenderCommandPipeFlags::None);

static TAutoConsoleVariable<int32> CVarRayTracingCableMeshes(
	TEXT("r.RayTracing.Geometry.Cable"),
	1,
	TEXT("Include Cable meshes in ray tracing effects (default = 1 (cable meshes enabled in ray tracing))"));

static TAutoConsoleVariable<int32> CVarRayTracingCableMeshesWPO(
	TEXT("r.RayTracing.Geometry.Cable.WPO"),
	1,
	TEXT("World position offset evaluation for cable meshes with EvaluateWPO enabled in ray tracing effects.\n")
	TEXT(" 0: Cable meshes with world position offset visible in ray tracing, WPO evaluation disabled.\n")
	TEXT(" 1: Cable meshes with world position offset visible in ray tracing, WPO evaluation enabled (default).\n")
);

static TAutoConsoleVariable<int32> CVarRayTracingCableMeshesWPOCulling(
	TEXT("r.RayTracing.Geometry.Cable.WPO.Culling"),
	1,
	TEXT("Enable culling for WPO evaluation for cable meshes in ray tracing (default = 1 (Culling enabled))"));

static TAutoConsoleVariable<float> CVarRayTracingCableMeshesWPOCullingRadius(
	TEXT("r.RayTracing.Geometry.Cable.WPO.CullingRadius"),
	12000.0f, // 120 m
	TEXT("Do not evaluate world position offset for cable meshes outside of this radius in ray tracing effects (default = 12000 (120m))"));

#include UE_INLINE_GENERATED_CPP_BY_NAME(CableComponent)

DECLARE_CYCLE_STAT(TEXT("Cable Sim"), STAT_Cable_SimTime, STATGROUP_CableComponent);
DECLARE_CYCLE_STAT(TEXT("Cable Solve"), STAT_Cable_SolveTime, STATGROUP_CableComponent);
DECLARE_CYCLE_STAT(TEXT("Cable Collision"), STAT_Cable_CollisionTime, STATGROUP_CableComponent);
DECLARE_CYCLE_STAT(TEXT("Cable Integrate"), STAT_Cable_IntegrateTime, STATGROUP_CableComponent);

static FName CableEndSocketName(TEXT("CableEnd"));
static FName CableStartSocketName(TEXT("CableStart"));

//////////////////////////////////////////////////////////////////////////

/** Index Buffer */
class FCableIndexBuffer : public FIndexBuffer 
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateIndex<int32>(TEXT("FCableIndexBuffer"), NumIndices)
			.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
			.DetermineInitialState();

		IndexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
	}

	int32 NumIndices;
};

/** Dynamic data sent to render thread */
struct FCableDynamicData
{
	/** Array of points */
	TArray<FVector> CablePoints;
};

//////////////////////////////////////////////////////////////////////////
// FCableSceneProxy

class FCableSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FCableSceneProxy(UCableComponent* Component)
		: FPrimitiveSceneProxy(Component)
		, Material(NULL)
		, VertexFactory(GetScene().GetFeatureLevel(), "FCableSceneProxy")
		, MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform()))
		, NumSegments(Component->NumSegments)
		, CableWidth(Component->CableWidth)
		, NumSides(Component->NumSides)
		, TileMaterial(Component->TileMaterial)
	{
		VertexBuffers.InitWithDummyData(&UE::RenderCommandPipe::Cable, &VertexFactory, GetRequiredVertexCount());

		IndexBuffer.NumIndices = GetRequiredIndexCount();

		// Grab material
		Material = Component->GetMaterial(0);
		if(Material == NULL)
		{
			Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}

#if RHI_RAYTRACING
		bSupportRayTracing = IsRayTracingEnabled();
		bDynamicRayTracingGeometry = bSupportRayTracing && MaterialRelevance.bUsesWorldPositionOffset;
#endif

		ENQUEUE_RENDER_COMMAND(InitCableResources)(UE::RenderCommandPipe::Cable,
			[this] (FRHICommandList& RHICmdList)
		{
			IndexBuffer.InitResource(RHICmdList);

#if RHI_RAYTRACING
			if (bSupportRayTracing)
			{
				CreateStaticRayTracingGeometry(RHICmdList);
				bNeedsToUpdateRayTracingCache = true;

				if (bDynamicRayTracingGeometry)
				{
					CreateDynamicRayTracingGeometries(RHICmdList);
				}
			}
#endif // RHI_RAYTRACING
		});
	}

	virtual ~FCableSceneProxy()
	{
		VertexBuffers.PositionVertexBuffer.ReleaseResource();
		VertexBuffers.StaticMeshVertexBuffer.ReleaseResource();
		VertexBuffers.ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
		VertexFactory.ReleaseResource();

#if RHI_RAYTRACING
		StaticRayTracingGeometry.ReleaseResource();
		DynamicRayTracingGeometry.ReleaseResource();
#endif
	}

	int32 GetRequiredVertexCount() const
	{
		return (NumSegments + 1) * (NumSides + 1);
	}

	int32 GetRequiredIndexCount() const
	{
		return (NumSegments * NumSides * 2) * 3;
	}

	int32 GetVertIndex(int32 AlongIdx, int32 AroundIdx) const
	{
		return (AlongIdx * (NumSides+1)) + AroundIdx;
	}

	void BuildCableMesh(FRHICommandListBase& RHICmdList, const FCableDynamicData* NewDynamicData)
	{
		const TArray<FVector>& InPoints = NewDynamicData->CablePoints;
		const FColor VertexColor(255, 255, 255);
		const int32 NumPoints = InPoints.Num();
		const int32 SegmentCount = NumPoints - 1;

		// Build vertices

		// We double up the first and last vert of the ring, because the UVs are different
		int32 NumRingVerts = NumSides + 1;

		uint32 VertIdx = 0;

		// For each point along spline..
		for (int32 PointIdx = 0; PointIdx < NumPoints; PointIdx++)
		{
			const float AlongFrac = (float)PointIdx / (float)SegmentCount; // Distance along cable

			// Find direction of cable at this point, by averaging previous and next points
			const int32 PrevIndex = FMath::Max(0, PointIdx - 1);
			const int32 NextIndex = FMath::Min(PointIdx + 1, NumPoints - 1);
			const FVector ForwardDir = (InPoints[NextIndex] - InPoints[PrevIndex]).GetSafeNormal();

			// Find quat from up (Z) vector to forward
			const FQuat DeltaQuat = FQuat::FindBetween(FVector(0, 0, -1), ForwardDir);

			// Apply quat orth vectors
			const FVector RightDir = DeltaQuat.RotateVector(FVector(0, 1, 0));
			const FVector UpDir = DeltaQuat.RotateVector(FVector(1, 0, 0));

			// Generate a ring of verts
			for (int32 RingVertIdx = 0; RingVertIdx < NumRingVerts; RingVertIdx++)
			{
				const float AroundFrac = float(RingVertIdx) / float(NumSides);
				// Find angle around the ring
				const float RadAngle = 2.f * PI * AroundFrac;
				// Find direction from center of cable to this vertex
				const FVector OutDir = (FMath::Cos(RadAngle) * UpDir) + (FMath::Sin(RadAngle) * RightDir);

				VertexBuffers.PositionVertexBuffer.VertexPosition(VertIdx) = FVector3f(InPoints[PointIdx] + (OutDir * 0.5f * CableWidth));
				VertexBuffers.StaticMeshVertexBuffer.SetVertexUV(VertIdx, 0, FVector2f(AlongFrac * TileMaterial, AroundFrac));
				VertexBuffers.ColorVertexBuffer.VertexColor(VertIdx) = VertexColor;
				VertexBuffers.StaticMeshVertexBuffer.SetVertexTangents(VertIdx, (FVector3f)ForwardDir, FVector3f(OutDir ^ ForwardDir), (FVector3f)OutDir);
				VertIdx++;
			}
		}

		check(VertIdx == GetRequiredVertexCount());

		{
			FPositionVertexBuffer& VertexBuffer = VertexBuffers.PositionVertexBuffer;
			void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetNumVertices() * VertexBuffer.GetStride(), RLM_WriteOnly);
			FMemory::Memcpy(VertexBufferData, VertexBuffer.GetVertexData(), VertexBuffer.GetNumVertices() * VertexBuffer.GetStride());
			RHICmdList.UnlockBuffer(VertexBuffer.VertexBufferRHI);
		}

		{
			FColorVertexBuffer& ColorBuffer = VertexBuffers.ColorVertexBuffer;
			void* VertexBufferData = RHICmdList.LockBuffer(ColorBuffer.VertexBufferRHI, 0, ColorBuffer.GetNumVertices() * ColorBuffer.GetStride(), RLM_WriteOnly);
			FMemory::Memcpy(VertexBufferData, ColorBuffer.GetVertexData(), ColorBuffer.GetNumVertices() * ColorBuffer.GetStride());
			RHICmdList.UnlockBuffer(ColorBuffer.VertexBufferRHI);
		}

		{
			FStaticMeshVertexBuffer& VertexBuffer = VertexBuffers.StaticMeshVertexBuffer;
			void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.TangentsVertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetTangentSize(), RLM_WriteOnly);
			FMemory::Memcpy(VertexBufferData, VertexBuffer.GetTangentData(), VertexBuffer.GetTangentSize());
			RHICmdList.UnlockBuffer(VertexBuffer.TangentsVertexBuffer.VertexBufferRHI);
		}

		{
			FStaticMeshVertexBuffer& VertexBuffer = VertexBuffers.StaticMeshVertexBuffer;
			void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.TexCoordVertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetTexCoordSize(), RLM_WriteOnly);
			FMemory::Memcpy(VertexBufferData, VertexBuffer.GetTexCoordData(), VertexBuffer.GetTexCoordSize());
			RHICmdList.UnlockBuffer(VertexBuffer.TexCoordVertexBuffer.VertexBufferRHI);
		}

		int32* IndexBufferData = static_cast<int32*>(RHICmdList.LockBuffer(IndexBuffer.IndexBufferRHI, 0, GetRequiredIndexCount() * sizeof(int32), RLM_WriteOnly));
		uint32 IndexBufferIndex = 0;

		// Build triangles
		for (int32 SegIdx = 0; SegIdx < SegmentCount; SegIdx++)
		{
			for (int32 SideIdx = 0; SideIdx < NumSides; SideIdx++)
			{
				int32 TL = GetVertIndex(SegIdx, SideIdx);
				int32 BL = GetVertIndex(SegIdx, SideIdx + 1);
				int32 TR = GetVertIndex(SegIdx + 1, SideIdx);
				int32 BR = GetVertIndex(SegIdx + 1, SideIdx + 1);

				IndexBufferData[IndexBufferIndex++] = TL;
				IndexBufferData[IndexBufferIndex++] = BL;
				IndexBufferData[IndexBufferIndex++] = TR;

				IndexBufferData[IndexBufferIndex++] = TR;
				IndexBufferData[IndexBufferIndex++] = BL;
				IndexBufferData[IndexBufferIndex++] = BR;
			}
		}

		RHICmdList.UnlockBuffer(IndexBuffer.IndexBufferRHI);

		check(IndexBufferIndex == GetRequiredIndexCount());
	}

	/** Called on render thread to assign new dynamic data */
	void SetDynamicData_RenderThread(FRHICommandListBase& RHICmdList, FCableDynamicData* NewDynamicData)
	{
		if (NewDynamicData != nullptr)
		{
			// Build mesh from cable points
			BuildCableMesh(RHICmdList, NewDynamicData);

#if RHI_RAYTRACING
			if (StaticRayTracingGeometry.IsInitialized())
			{
				if (StaticRayTracingGeometry.IsEvicted())
				{
					StaticRayTracingGeometry.CreateRayTracingGeometry(RHICmdList, ERTAccelerationStructureBuildPriority::Normal);
				}
				else if(!StaticRayTracingGeometry.HasPendingBuildRequest())
				{
					StaticRayTracingGeometry.SetRequiresBuild(true);
					StaticRayTracingGeometry.RequestBuildIfNeeded(RHICmdList, ERTAccelerationStructureBuildPriority::Normal);
				}

				bNeedsToUpdateRayTracingCache = true;
			}
#endif

			delete NewDynamicData;
			NewDynamicData = NULL;
		}
	}

	virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override
	{
		checkSlow(IsInParallelRenderingThread());

		if (!HasViewDependentDPG())
		{
			FMeshBatch Mesh;
			Mesh.VertexFactory = &VertexFactory;
			Mesh.MaterialRenderProxy = Material->GetRenderProxy();
			Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
			Mesh.Type = PT_TriangleList;
			Mesh.DepthPriorityGroup = SDPG_World;
			Mesh.MeshIdInPrimitive = 0;
			Mesh.LODIndex = 0;
			Mesh.SegmentIndex = 0;

			FMeshBatchElement& BatchElement = Mesh.Elements[0];
			BatchElement.IndexBuffer = &IndexBuffer;
			BatchElement.FirstIndex = 0;
			BatchElement.NumPrimitives = GetRequiredIndexCount() / 3;
			BatchElement.MinVertexIndex = 0;
			BatchElement.MaxVertexIndex = GetRequiredVertexCount();

			PDI->DrawMesh(Mesh, FLT_MAX);
		}
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		QUICK_SCOPE_CYCLE_COUNTER( STAT_CableSceneProxy_GetDynamicMeshElements );

		const bool bWireframe = AllowDebugViewmodes() && ViewFamily.EngineShowFlags.Wireframe;

		auto WireframeMaterialInstance = new FColoredMaterialRenderProxy(
			GEngine->WireframeMaterial ? GEngine->WireframeMaterial->GetRenderProxy() : NULL,
			FLinearColor(0, 0.5f, 1.f)
			);

		Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);

		FMaterialRenderProxy* MaterialProxy = NULL;
		if(bWireframe)
		{
			MaterialProxy = WireframeMaterialInstance;
		}
		else
		{
			MaterialProxy = Material->GetRenderProxy();
		}

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];
				// Draw the mesh.
				FMeshBatch& Mesh = Collector.AllocateMesh();
				FMeshBatchElement& BatchElement = Mesh.Elements[0];
				BatchElement.IndexBuffer = &IndexBuffer;
				Mesh.bWireframe = bWireframe;
				Mesh.VertexFactory = &VertexFactory;
				Mesh.MaterialRenderProxy = MaterialProxy;

				FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
				FPrimitiveUniformShaderParametersBuilder Builder;
				BuildUniformShaderParameters(Builder);
				DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);

				BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

				BatchElement.FirstIndex = 0;
				BatchElement.NumPrimitives = GetRequiredIndexCount()/3;
				BatchElement.MinVertexIndex = 0;
				BatchElement.MaxVertexIndex = GetRequiredVertexCount();
				Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
				Mesh.Type = PT_TriangleList;
				Mesh.DepthPriorityGroup = SDPG_World;
				Mesh.bCanApplyViewModeOverrides = false;
				Collector.AddMesh(ViewIndex, Mesh);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				// Render bounds
				RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
#endif
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		const bool bAllowStaticLighting = IsStaticLightingAllowed();
		if (
#if !(UE_BUILD_SHIPPING) || WITH_EDITOR
			IsRichView(*View->Family) ||
			View->Family->EngineShowFlags.Collision ||
			View->Family->EngineShowFlags.Bounds ||
			View->Family->EngineShowFlags.VisualizeInstanceUpdates ||
#endif
#if WITH_EDITOR
			(IsSelected() && View->Family->EngineShowFlags.VertexColors) ||
			(IsSelected() && View->Family->EngineShowFlags.PhysicalMaterialMasks) ||
#endif
			// Force down dynamic rendering path if invalid lightmap settings, so we can apply an error material in DrawRichMesh
			(bAllowStaticLighting && HasStaticLighting() && !HasValidSettingsForStaticLighting()) ||
			HasViewDependentDPG()
			)
		{
			Result.bDynamicRelevance = true;
		}
		else
		{
			Result.bStaticRelevance = true;

#if WITH_EDITOR
			//only check these in the editor
			Result.bEditorVisualizeLevelInstanceRelevance = IsEditingLevelInstanceChild();
			Result.bEditorStaticSelectionRelevance = (IsSelected() || IsHovered());
#endif
		}

		MaterialRelevance.SetPrimitiveViewRelevance(Result);

		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

		return Result;
	}

	virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }

	uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

#if RHI_RAYTRACING
	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override
	{
		if (CVarRayTracingCableMeshes.GetValueOnRenderThread() == 0)
		{
			return;
		}

		if (!ensureMsgf(IsRayTracingRelevant(),
			TEXT("GetDynamicRayTracingInstances() is only expected to be called for scene proxies that are compatible with ray tracing. ")
			TEXT("RT-relevant primitive gathering code in FDeferredShadingSceneRenderer may be wrong.")))
		{
			return;
		}

		TConstArrayView<const FSceneView*> Views = Collector.GetViews();
		const uint32 VisibilityMap = Collector.GetVisibilityMap();

		// RT geometry will be generated based on first active view and then reused for all other views
		// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
		const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
		checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

		const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

		bool bEvaluateWPO = bDynamicRayTracingGeometry && CVarRayTracingCableMeshesWPO.GetValueOnRenderThread() == 1;
		
		if (bEvaluateWPO && CVarRayTracingCableMeshesWPOCulling.GetValueOnRenderThread() > 0)
		{
			const FVector MeshCenter = GetBounds().Origin;
			const float CullingRadius = CVarRayTracingCableMeshesWPOCullingRadius.GetValueOnRenderThread();
			const float BoundingRadius = GetBounds().SphereRadius;

			bEvaluateWPO = false;
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if ((VisibilityMap & (1 << ViewIndex)) == 0)
				{
					continue;
				}

				const FSceneView* View = Views[ViewIndex];

				const FVector ViewCenter = View->ViewMatrices.GetViewOrigin();
				
				if (FVector(ViewCenter - MeshCenter).Size() <= (CullingRadius + BoundingRadius))
				{
					bEvaluateWPO |= true;
				}
			}
		}
		
		if (!bEvaluateWPO)
		{
			if (!StaticRayTracingGeometry.IsValid())
			{
				return;
			}

			if (StaticRayTracingGeometry.IsEvicted())
			{
				Collector.AddReferencedGeometry(StaticRayTracingGeometry.GetGeometryHandle());
				return;
			}
		}

		FRayTracingGeometry& Geometry = bEvaluateWPO ? DynamicRayTracingGeometry : StaticRayTracingGeometry;

		if (Geometry.GetInitializer().TotalPrimitiveCount <= 0)
		{
			return;
		}

		FRayTracingInstance RayTracingInstance;

		const int32 NumRayTracingMaterialEntries = 1;

		if (bNeedsToUpdateRayTracingCache)
		{
			CachedRayTracingMaterials.Reset();
			CachedRayTracingMaterials.Reserve(NumRayTracingMaterialEntries);

			FMeshBatch& MeshBatch = CachedRayTracingMaterials.AddDefaulted_GetRef();

			MeshBatch.VertexFactory = &VertexFactory;
			MeshBatch.MaterialRenderProxy = Material->GetRenderProxy();
			MeshBatch.SegmentIndex = 0;
			MeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative();
			MeshBatch.Type = PT_TriangleList;
			MeshBatch.DepthPriorityGroup = SDPG_World;
			MeshBatch.bCanApplyViewModeOverrides = false;
			MeshBatch.CastRayTracedShadow = IsShadowCast(FirstActiveView);
			MeshBatch.DepthPriorityGroup = GetStaticDepthPriorityGroup();

			FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
			BatchElement.IndexBuffer = &IndexBuffer;

			FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
			FPrimitiveUniformShaderParametersBuilder Builder;
			BuildUniformShaderParameters(Builder);
			DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);

			BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;
			BatchElement.FirstIndex = 0;
			BatchElement.NumPrimitives = GetRequiredIndexCount() / 3;
			BatchElement.MinVertexIndex = 0;
			BatchElement.MaxVertexIndex = GetRequiredVertexCount();


			RayTracingInstance.MaterialsView = MakeArrayView(CachedRayTracingMaterials);
			bNeedsToUpdateRayTracingCache = false;
		}
		else
		{
			RayTracingInstance.MaterialsView = MakeArrayView(CachedRayTracingMaterials);

			// Skip computing the mask and flags in the renderer since we are using cached values.
			RayTracingInstance.bInstanceMaskAndFlagsDirty = false;
		}

		RayTracingInstance.Geometry = &Geometry;
		const FMatrix& ThisLocalToWorld = GetLocalToWorld();
		RayTracingInstance.InstanceTransformsView = MakeArrayView(&ThisLocalToWorld, 1);

		// TODO: Checking if VertexFactory.GetType()->SupportsRayTracingDynamicGeometry() should be done when initializing bDynamicRayTracingGeometry otherwise we end up with unbuilt BLAS
		if (bEvaluateWPO && VertexFactory.GetType()->SupportsRayTracingDynamicGeometry())
		{
			// Use the shared vertex buffer - needs to be updated every frame
			FRWBuffer* VertexBuffer = nullptr;

			const uint32 VertexCount = VertexBuffers.PositionVertexBuffer.GetNumVertices() + 1;

			{
				FRayTracingDynamicGeometryUpdateParams UpdateParams;
				UpdateParams.MeshBatchesView = CachedRayTracingMaterials;
				UpdateParams.NumVertices = (uint32)VertexCount;
				UpdateParams.VertexBufferSize = uint32((SIZE_T)VertexCount * sizeof(FVector3f));
				UpdateParams.NumTriangles = Geometry.GetInitializer().TotalPrimitiveCount;
				UpdateParams.Geometry = &Geometry;
				UpdateParams.Buffer = VertexBuffer;

				Collector.AddRayTracingGeometryUpdate(FirstActiveViewIndex, MoveTemp(UpdateParams));
			}
		}

		check(CachedRayTracingMaterials.Num() == RayTracingInstance.GetMaterials().Num());
		checkf(RayTracingInstance.Geometry->GetInitializer().Segments.Num() == CachedRayTracingMaterials.Num(), TEXT("Segments/Materials mismatch. Number of segments: %d. Number of Materials: %d."),
			RayTracingInstance.Geometry->GetInitializer().Segments.Num(),
			CachedRayTracingMaterials.Num());

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}

			Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
		}
	}

	virtual bool HasRayTracingRepresentation() const override { return bSupportRayTracing; }
	virtual bool IsRayTracingRelevant() const override { return true; }
	virtual bool IsRayTracingStaticRelevant() const override { return false; }

	void CreateStaticRayTracingGeometry(FRHICommandListBase& RHICmdList)
	{
		FRayTracingGeometryInitializer Initializer;
		static const FName DebugName("FCableSceneProxy");
		static int32 DebugNumber = 0;
		Initializer.DebugName = FDebugName(DebugName, DebugNumber++);
		Initializer.IndexBuffer = IndexBuffer.IndexBufferRHI;
		Initializer.TotalPrimitiveCount = IndexBuffer.NumIndices / 3;
		Initializer.GeometryType = RTGT_Triangles;
		Initializer.bFastBuild = true;
		Initializer.bAllowUpdate = false;

		FRayTracingGeometrySegment Segment;
		Segment.VertexBuffer = VertexBuffers.PositionVertexBuffer.VertexBufferRHI;
		Segment.NumPrimitives = Initializer.TotalPrimitiveCount;
		Segment.MaxVertices = VertexBuffers.PositionVertexBuffer.GetNumVertices();
		Initializer.Segments.Add(Segment);

		StaticRayTracingGeometry.SetInitializer(Initializer);
		StaticRayTracingGeometry.InitResource(RHICmdList);
	}
#endif

private:

	UMaterialInterface* Material;
	FStaticMeshVertexBuffers VertexBuffers;
	FCableIndexBuffer IndexBuffer;
	FLocalVertexFactory VertexFactory;

	FMaterialRelevance MaterialRelevance;

	int32 NumSegments;

	float CableWidth;

	int32 NumSides;

	float TileMaterial;

#if RHI_RAYTRACING
	void CreateDynamicRayTracingGeometries(FRHICommandListBase& RHICmdList)
	{
		FRayTracingGeometryInitializer Initializer = StaticRayTracingGeometry.GetInitializer();
		for (FRayTracingGeometrySegment& Segment : Initializer.Segments)
		{
			Segment.VertexBuffer = nullptr;
		}
		Initializer.bAllowUpdate = true;
		Initializer.bFastBuild = true;
		Initializer.Type = ERayTracingGeometryInitializerType::Rendering;

		DynamicRayTracingGeometry.SetInitializer(MoveTemp(Initializer));
		DynamicRayTracingGeometry.InitResource(RHICmdList);
	}

	bool bSupportRayTracing : 1;
	bool bDynamicRayTracingGeometry : 1;
	bool bNeedsToUpdateRayTracingCache : 1;

	FRayTracingGeometry StaticRayTracingGeometry;
	FRayTracingGeometry DynamicRayTracingGeometry;
	TArray<FMeshBatch> CachedRayTracingMaterials;
#endif
};



//////////////////////////////////////////////////////////////////////////

UCableComponent::UCableComponent( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	PrimaryComponentTick.bCanEverTick = true;
	bTickInEditor = true;
	bAutoActivate = true;

	bAttachStart = true;
	bAttachEnd = true;
	CableWidth = 10.f;
	NumSegments = 10;
	NumSides = 4;
	EndLocation = FVector(100.f,0,0);
	CableLength = 100.f;
	SubstepTime = 0.02f;
	SolverIterations = 1;
	TileMaterial = 1.f;
	CollisionFriction = 0.2f;
	CableGravityScale = 1.f;

	GetEndPositions(LastStartPoint,LastEndPoint);
	LastTransform = GetComponentTransform();
	LastEndPointTransform = GetComponentTransform();
	LastEndLocation = EndLocation;
	bLastStartAttached = bAttachStart;
	bLastEndAttached = bAttachEnd;

	SetCollisionProfileName(UCollisionProfile::PhysicsActor_ProfileName);
}

FPrimitiveSceneProxy* UCableComponent::CreateSceneProxy()
{
	if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
	{
		return nullptr;
	}

	return new FCableSceneProxy(this);
}

void UCableComponent::PostLoad()
{
	Super::PostLoad();

	PrecachePSOs();
}

void UCableComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	FMaterialInterfacePSOPrecacheParams Params;
	Params.PSOPrecacheParams = BasePrecachePSOParams;
	Params.PSOPrecacheParams.bReverseCulling |= GetRenderMatrix().Determinant() < 0.f;
	//Matches the logic in FCableSceneProxy
	Params.VertexFactoryDataList.Add(&FLocalVertexFactory::StaticType);
	Params.MaterialInterface = GetMaterial(0);
	if (!Params.MaterialInterface)
	{
		Params.MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
	}
	OutParams.Add(Params);
}

int32 UCableComponent::GetNumMaterials() const
{
	return 1;
}

void UCableComponent::OnRegister()
{
	Super::OnRegister();

	const int32 NumParticles = NumSegments+1;

	Particles.Reset();
	Particles.AddUninitialized(NumParticles);

	FVector CableStart, CableEnd;
	GetEndPositions(CableStart, CableEnd);

	const FVector Delta = CableEnd - CableStart;

	for(int32 ParticleIdx=0; ParticleIdx<NumParticles; ParticleIdx++)
	{
		FCableParticle& Particle = Particles[ParticleIdx];

		const float Alpha = (float)ParticleIdx/(float)NumSegments;
		const FVector InitialPosition = CableStart + (Alpha * Delta);

		Particle.Position = InitialPosition;
		Particle.OldPosition = InitialPosition;
		Particle.bFree = true; // default to free, will be fixed if desired in TickComponent
	}

	LastStartPoint = CableStart;
	LastEndPoint = CableEnd;
	LastEndLocation = EndLocation;
	LastTransform = GetComponentTransform();
	USceneComponent* EndComponent = Cast<USceneComponent>(AttachEndTo.GetComponent(GetOwner()));
	if (EndComponent == NULL)
	{
		EndComponent = this;
	}

	if (AttachEndToSocketName != NAME_None)
	{
		LastEndPointTransform = EndComponent->GetSocketTransform(AttachEndToSocketName);
	}
	else
	{
		LastEndPointTransform = EndComponent->GetComponentTransform();
	}
	bLastStartAttached = bAttachStart;
	bLastEndAttached = bAttachEnd;

	PrecachePSOs();
}

void UCableComponent::VerletIntegrate(float InSubstepTime, const FVector& Gravity)
{
	SCOPE_CYCLE_COUNTER(STAT_Cable_IntegrateTime);

	const int32 NumParticles = NumSegments+1;
	const float SubstepTimeSqr = InSubstepTime * InSubstepTime;

	for(int32 ParticleIdx=0; ParticleIdx<NumParticles; ParticleIdx++)
	{
		FCableParticle& Particle = Particles[ParticleIdx];
		if(Particle.bFree)
		{
			// Calc overall force
			const FVector ParticleForce = Gravity + CableForce;

			// Find vel
			const FVector Vel = Particle.Position - Particle.OldPosition;
			// Update position
			const FVector NewPosition = Particle.Position + Vel + (SubstepTimeSqr * ParticleForce);

			Particle.OldPosition = Particle.Position;
			Particle.Position = NewPosition;
		}
	}
}

/** Solve a single distance constraint between a pair of particles */
static FORCEINLINE void SolveDistanceConstraint(FCableParticle& ParticleA, FCableParticle& ParticleB, float DesiredDistance)
{
	// Find current vector between particles
	FVector Delta = ParticleB.Position - ParticleA.Position;
	float CurrentDistance = Delta.Size();

	bool bNormalizedOK = Delta.Normalize();

	// If particles are right on top of each other, separate with an abitrarily-chosen direction
	FVector CorrectionDirection = bNormalizedOK ? Delta : FVector{ 1, 0, 0 };
	FVector VectorCorrection = (CurrentDistance - DesiredDistance) * CorrectionDirection;

	// Only move free particles to satisfy constraints
	if(ParticleA.bFree && ParticleB.bFree)
	{
		ParticleA.Position += 0.5f * VectorCorrection;
		ParticleB.Position -= 0.5f * VectorCorrection;
	}
	else if(ParticleA.bFree)
	{
		ParticleA.Position += VectorCorrection;
	}
	else if(ParticleB.bFree)
	{
		ParticleB.Position -= VectorCorrection;
	}
}

void UCableComponent::SolveConstraints()
{
	SCOPE_CYCLE_COUNTER(STAT_Cable_SolveTime);

	const float SegmentLength = CableLength/(float)NumSegments;

	// For each iteration..
	for (int32 IterationIdx = 0; IterationIdx < SolverIterations; IterationIdx++)
	{
		// Solve distance constraint for each segment
		for (int32 SegIdx = 0; SegIdx < NumSegments; SegIdx++)
		{
			FCableParticle& ParticleA = Particles[SegIdx];
			FCableParticle& ParticleB = Particles[SegIdx + 1];
			// Solve for this pair of particles
			SolveDistanceConstraint(ParticleA, ParticleB, SegmentLength);
		}

		// If desired, solve stiffness constraints (distance constraints between every other particle)
		if (bEnableStiffness)
		{
			for (int32 SegIdx = 0; SegIdx < NumSegments-1; SegIdx++)
			{
				FCableParticle& ParticleA = Particles[SegIdx];
				FCableParticle& ParticleB = Particles[SegIdx + 2];
				SolveDistanceConstraint(ParticleA, ParticleB, 2.f*SegmentLength);
			}
		}
	}
}

void UCableComponent::PerformCableCollision()
{
	SCOPE_CYCLE_COUNTER(STAT_Cable_CollisionTime);

	UWorld* World = GetWorld();
	// If we have a world, and collision is not disabled
	if (World && GetCollisionEnabled() != ECollisionEnabled::NoCollision)
	{
		// Get collision settings from component
		FCollisionQueryParams Params(SCENE_QUERY_STAT(CableCollision));

		ECollisionChannel TraceChannel = GetCollisionObjectType();
		FCollisionResponseParams ResponseParams(GetCollisionResponseToChannels());

		// Iterate over each particle
		for (int32 ParticleIdx = 0; ParticleIdx < Particles.Num(); ParticleIdx++)
		{
			FCableParticle& Particle = Particles[ParticleIdx];
			// If particle is free
			if (Particle.bFree)
			{
				// Do sphere sweep
				FHitResult Result;
				bool bHit = World->SweepSingleByChannel(Result, Particle.OldPosition, Particle.Position, FQuat::Identity, TraceChannel, FCollisionShape::MakeSphere(0.5f * CableWidth), Params, ResponseParams);
				// If we got a hit, resolve it
				if (bHit)
				{
					if (Result.bStartPenetrating)
					{
						Particle.Position += (Result.Normal * Result.PenetrationDepth);
					}
					else
					{
						Particle.Position = Result.Location;
					}

					// Find new velocity, after fixing collision
					FVector Delta = Particle.Position - Particle.OldPosition;
					// Find component in normal
					float NormalDelta = Delta | Result.Normal;
					// Find component in plane
					FVector PlaneDelta = Delta - (NormalDelta * Result.Normal);

					// Zero out any positive separation velocity, basically zero restitution
					Particle.OldPosition += (NormalDelta * Result.Normal);

					// Apply friction in plane of collision if desired
					if (CollisionFriction > KINDA_SMALL_NUMBER)
					{
						// Scale plane delta  by 'friction'
						FVector ScaledPlaneDelta = PlaneDelta * CollisionFriction;

						// Apply delta to old position reduce implied velocity in collision plane
						Particle.OldPosition += ScaledPlaneDelta;
					}
				}
			}
		}
	}
}

void UCableComponent::PerformSubstep(float InSubstepTime, const FVector& Gravity)
{
	SCOPE_CYCLE_COUNTER(STAT_Cable_SimTime);

	VerletIntegrate(InSubstepTime, Gravity);

	SolveConstraints();

	if (bEnableCollision)
	{
		PerformCableCollision();
	}
}

void UCableComponent::SetAttachEndToComponent(USceneComponent* Component, FName SocketName)
{
	AttachEndTo.OtherActor = Component ? Component->GetOwner() : nullptr;
	AttachEndTo.ComponentProperty = NAME_None;
	AttachEndTo.OverrideComponent = Component;
	AttachEndToSocketName = SocketName;
}

void UCableComponent::SetAttachEndTo(AActor* Actor, FName ComponentProperty, FName SocketName)
{
	AttachEndTo.OtherActor = Actor;
	AttachEndTo.ComponentProperty = ComponentProperty;
	AttachEndToSocketName = SocketName;
}

AActor* UCableComponent::GetAttachedActor() const
{
	return AttachEndTo.OtherActor.Get();
}

USceneComponent* UCableComponent::GetAttachedComponent() const
{
	return Cast<USceneComponent>(AttachEndTo.GetComponent(GetOwner()));
}

void UCableComponent::GetCableParticleLocations(TArray<FVector>& Locations) const
{
	Locations.Empty();
	for (const FCableParticle& Particle : Particles)
	{
		Locations.Add(Particle.Position);
	}
}

void UCableComponent::DoTeleportCorrections(const FVector& StartPosition, const FVector& EndPosition)
{
	const FTransform ComponentTransform = GetComponentTransform();
	USceneComponent* EndComponent = Cast<USceneComponent>(AttachEndTo.GetComponent(GetOwner()));
	if (EndComponent == NULL)
	{
		EndComponent = this;
	}

	FTransform EndPointTransform = FTransform::Identity;

	if (AttachEndToSocketName != NAME_None)
	{
		EndPointTransform = EndComponent->GetSocketTransform(AttachEndToSocketName);
	}
	else
	{
		EndPointTransform = EndComponent->GetComponentTransform();
	}

	bool bShouldCorrect = false;
	const bool bReattached = (bAttachStart && !bLastStartAttached) || (bAttachEnd && !bLastEndAttached);
	const bool bAttachmentChange = (bAttachStart != bLastStartAttached) || (bAttachEnd != bLastEndAttached);

	//Cannot apply corrections unless some end of the cable is not free falling.
	if (bAttachStart || bAttachEnd)
	{
		if (bReattached && bTeleportAfterReattach)
		{
			bShouldCorrect = true;
		}
		else
		{
			const float SquaredDistanceThreshold = TeleportDistanceThreshold * TeleportDistanceThreshold;

			const float StartDifference = FVector::DistSquared(StartPosition, LastStartPoint);
			const float EndDifference = FVector::DistSquared(EndPosition, LastEndPoint);

			if (SquaredDistanceThreshold > 0 && (StartDifference > SquaredDistanceThreshold || EndDifference > SquaredDistanceThreshold))
			{
				bShouldCorrect = true;
			}
			else if(!bAttachmentChange && !(bAttachStart && bAttachEnd && EndComponent != this))
			{
				//Only calculate rotational difference if we have two consecutive frames with transform data. 
				//Don't attempt to calculate rotation if the end points can be controlled by separate transforms.

				const FTransform CurrentRootTransform = bAttachEnd ?  EndPointTransform : ComponentTransform;
				const FTransform LastRootTransform = bAttachEnd ? LastEndPointTransform : LastTransform;

				const float RotationDifference = CurrentRootTransform.GetRotation().AngularDistance(LastRootTransform.GetRotation());

				if (TeleportRotationThreshold > 0 && FMath::RadiansToDegrees(FMath::Abs(RotationDifference)) > TeleportRotationThreshold)
				{
					bShouldCorrect = true;
				}
			}
		}
	}

	if (bShouldCorrect)
	{
		bool bCorrectiveTransform = false;
		bool bCorrectiveReinitialize = false;
		bool bAdjustStart = false;
		bool bAdjustEnd = false;

		FTransform DeltaTransform = FTransform::Identity;
		const FVector DeltaEndLocation = EndLocation - LastEndLocation;

		if (bAttachmentChange || bResetAfterTeleport)
		{
			bCorrectiveReinitialize = true;
		}
		else
		{
			if (bAttachStart && bAttachEnd)
			{
				if (DeltaEndLocation.SquaredLength() > 0 || EndComponent != this)
				{
					//If the end points can change independently of the starts we conservatively reinitialize
					bCorrectiveReinitialize = true;
				}
				else
				{
					//Start and end points will both move using the cable transform. 
					DeltaTransform = (LastTransform.Inverse() * ComponentTransform);
					bCorrectiveTransform = true;
				}
			}
			else
			{
				bCorrectiveTransform = true;
				if (!bAttachStart)
				{
					//Detached start: Use cable transform to correct if the cable isn't attached to another object, otherwise use the transform of that object.
					const FTransform CurrentRootTransform = EndComponent == this ? ComponentTransform : EndPointTransform;
					const FTransform LastRootTransform = EndComponent == this ? LastTransform : LastEndPointTransform;

					DeltaTransform = FTransform(CurrentRootTransform.TransformVector(DeltaEndLocation)) * (LastRootTransform.Inverse() * CurrentRootTransform);
					bAdjustStart = true;
				}
				else if (!bAttachEnd)
				{
					//Detached end: Use cable transform to correct.
					DeltaTransform = (LastTransform.Inverse() * ComponentTransform);
					bAdjustEnd = true;
				}
			}
		}

		if (bCorrectiveTransform)
		{
			int Index = bAdjustStart ? 0 : 1;
			const int LastIndex = bAdjustEnd ? NumSegments : NumSegments - 1;

			for (; Index <= LastIndex; Index++)
			{
				Particles[Index].Position = DeltaTransform.TransformPosition(Particles[Index].Position);
				Particles[Index].OldPosition = DeltaTransform.TransformPosition(Particles[Index].OldPosition);
			}
		}
		else if (bCorrectiveReinitialize)
		{
			const FVector Delta = EndPosition - StartPosition;
			for (int32 ParticleIdx = 0; ParticleIdx <= NumSegments; ParticleIdx++)
			{
				FCableParticle& Particle = Particles[ParticleIdx];

				const float Alpha = (float)ParticleIdx / (float)NumSegments;
				const FVector InitialPosition = StartPosition + (Alpha * Delta);

				Particle.Position = InitialPosition;
				Particle.OldPosition = InitialPosition;
			}
		}
	}

	LastTransform = ComponentTransform;
	LastEndPointTransform = EndPointTransform;
	LastEndLocation = EndLocation;
	LastStartPoint = StartPosition;
	LastEndPoint = EndPosition;
	bLastStartAttached = bAttachStart;
	bLastEndAttached = bAttachEnd;
}


void UCableComponent::GetEndPositions(FVector& OutStartPosition, FVector& OutEndPosition)
{
	// Start position is just component position
	OutStartPosition = GetComponentLocation();

	// See if we want to attach the other end to some other component
	USceneComponent* EndComponent = Cast<USceneComponent>(AttachEndTo.GetComponent(GetOwner()));
	if(EndComponent == NULL)
	{
		EndComponent = this;
	}

	if (AttachEndToSocketName != NAME_None)
	{
		OutEndPosition = EndComponent->GetSocketTransform(AttachEndToSocketName).TransformPosition(EndLocation);
	}
	else
	{
		OutEndPosition = EndComponent->GetComponentTransform().TransformPosition(EndLocation);
	}

}

void UCableComponent::OnVisibilityChanged()
{
	Super::OnVisibilityChanged();

	// Does not interact well with any other states that would be blocking tick
	if (bSkipCableUpdateWhenNotVisible)
	{
		SetComponentTickEnabled(IsVisible());
	}
}

void UCableComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (bSkipCableUpdateWhenNotVisible && !IsVisible())
	{
		SetComponentTickEnabled(false);
		return;
	}

	AActor* Owner = GetOwner();
	if (bSkipCableUpdateWhenNotOwnerRecentlyRendered && Owner && !Owner->WasRecentlyRendered(2.0f))
	{
		return;
	}

	const FVector Gravity = FVector(0, 0, GetWorld()->GetGravityZ()) * CableGravityScale;

	// Update end points
	FVector CableStart, CableEnd;
	GetEndPositions(CableStart, CableEnd);

	FCableParticle& StartParticle = Particles[0];

	if (bAttachStart)
	{
		StartParticle.Position = StartParticle.OldPosition = CableStart;
		StartParticle.bFree = false;
	}
	else
	{
		StartParticle.bFree = true;
	}

	FCableParticle& EndParticle = Particles[NumSegments];
	if (bAttachEnd)
	{
		EndParticle.Position = EndParticle.OldPosition = CableEnd;
		EndParticle.bFree = false;
	}
	else
	{
		EndParticle.bFree = true;
	}

	DoTeleportCorrections(CableStart, CableEnd);

	// Ensure a non-zero substep
	float UseSubstep = FMath::Max(SubstepTime, 0.005f);

	// Perform simulation substeps
	TimeRemainder += DeltaTime;
	while (TimeRemainder > UseSubstep)
	{

		PerformSubstep(bUseSubstepping ? UseSubstep : TimeRemainder, Gravity);
		if (bUseSubstepping)
		{
			TimeRemainder -= UseSubstep;
		}
		else
		{
			TimeRemainder = 0.0f;
		}
	}

	// Need to send new data to render thread
	MarkRenderDynamicDataDirty();

	// Call this because bounds have changed
	UpdateComponentToWorld();
};

void UCableComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);

	FRegisterComponentContext::SendRenderDynamicData(Context, this);
}

void UCableComponent::ApplyWorldOffset(const FVector & InOffset, bool bWorldShift)
{
	Super::ApplyWorldOffset(InOffset, bWorldShift);

	for (FCableParticle& Particle : Particles)
	{
		Particle.Position += InOffset;
		Particle.OldPosition += InOffset;
	}
}

void UCableComponent::SendRenderDynamicData_Concurrent()
{
	if(SceneProxy)
	{
		// Allocate cable dynamic data
		FCableDynamicData* DynamicData = new FCableDynamicData;

		// Transform current positions from particles into component-space array
		const FTransform& ComponentTransform = GetComponentTransform();
		int32 NumPoints = NumSegments+1;
		DynamicData->CablePoints.AddUninitialized(NumPoints);
		for(int32 PointIdx=0; PointIdx<NumPoints; PointIdx++)
		{
			DynamicData->CablePoints[PointIdx] = ComponentTransform.InverseTransformPosition(Particles[PointIdx].Position);
		}

		// Enqueue command to send to render thread
		FCableSceneProxy* CableSceneProxy = (FCableSceneProxy*)SceneProxy;
		ENQUEUE_RENDER_COMMAND(FSendCableDynamicData)(UE::RenderCommandPipe::Cable,
			[CableSceneProxy, DynamicData](FRHICommandListBase& RHICmdList)
		{
			CableSceneProxy->SetDynamicData_RenderThread(RHICmdList, DynamicData);
		});
	}
}

FBoxSphereBounds UCableComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	// Calculate bounding box of cable points
	FBox CableBox(ForceInit);

	const FTransform& ComponentTransform = GetComponentTransform();

	for(int32 ParticleIdx=0; ParticleIdx<Particles.Num(); ParticleIdx++)
	{
		const FCableParticle& Particle = Particles[ParticleIdx];
		CableBox += ComponentTransform.InverseTransformPosition(Particle.Position);
	}

	// Expand by cable radius (half cable width)
	return FBoxSphereBounds(CableBox.ExpandBy(0.5f * CableWidth)).TransformBy(LocalToWorld);
}

void UCableComponent::QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const 
{
	OutSockets.Add(FComponentSocketDescription(CableEndSocketName, EComponentSocketType::Socket));
	OutSockets.Add(FComponentSocketDescription(CableStartSocketName, EComponentSocketType::Socket));
}

FTransform UCableComponent::GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace) const
{
	int32 NumParticles = Particles.Num();
	if ((InSocketName == CableEndSocketName || InSocketName == CableStartSocketName) && NumParticles >= 2)
	{
		FVector ForwardDir, Pos;
		if (InSocketName == CableEndSocketName)
		{
			FVector LastPos = Particles[NumParticles - 1].Position;
			FVector PreviousPos = Particles[NumParticles - 2].Position;

			ForwardDir = (LastPos - PreviousPos).GetSafeNormal();
			Pos = LastPos;
		}
		else
		{
			FVector FirstPos = Particles[0].Position;
			FVector NextPos = Particles[1].Position;

			ForwardDir = (NextPos - FirstPos).GetSafeNormal();
			Pos = FirstPos;
		}

		const FQuat RotQuat = FQuat::FindBetween(FVector(1, 0, 0), ForwardDir);
		FTransform WorldSocketTM = FTransform(RotQuat, Pos, FVector(1, 1, 1));

		switch (TransformSpace)
		{
			case RTS_World:
			{
				return WorldSocketTM;
			}
			case RTS_Actor:
			{
				if (const AActor* Actor = GetOwner())
				{
					return WorldSocketTM.GetRelativeTransform(GetOwner()->GetTransform());
				}
				break;
			}
			case RTS_Component:
			{
				return WorldSocketTM.GetRelativeTransform(GetComponentTransform());
			}
		}
	}

	return Super::GetSocketTransform(InSocketName, TransformSpace);
}

bool UCableComponent::HasAnySockets() const
{
	return (Particles.Num() >= 2);
}

bool UCableComponent::DoesSocketExist(FName InSocketName) const
{
	return (InSocketName == CableEndSocketName) || (InSocketName == CableStartSocketName);
}

