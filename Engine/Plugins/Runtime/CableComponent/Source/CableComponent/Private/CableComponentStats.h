// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("Cable"), STATGROUP_CableComponent, STATCAT_Advanced);
